// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using Microsoft.Xbox.Services.DevTools.Authentication;
using Microsoft.Xbox.Services.DevTools.XblConfig;
using System.Xml.Linq;
using System.Reflection;
using System.IO;

namespace PartnerCenterTool
{
	class Program
	{
		static private DevAccount SignIn()
		{
			// show the login UI and check the result
			try
			{
				Tuple<DevAccount,string> Result = ToolAuthentication.SignInAsync(DevAccountSource.WindowsDevCenter, String.Empty).Result;
				return Result.Item1;
			}
			catch( Exception e )
			{
				Console.Error.WriteLine("Cannot sign in: {0}." + System.Environment.NewLine + System.Environment.NewLine + "Please check that you are using the correct account because Partner Center may not be using your normal Microsoft account depending on your organization.", GetExceptionString(e));
				return null;
			}
		}

		static void DumpAllProducts()
		{
			bool bFinished = false;

			while (!bFinished)
			{
				// log in to partner center
				DevAccount Account = SignIn();
				if (Account == null)
				{
					return;
				}

				// build xml product list
				try
				{
					// query products
					IEnumerable<Product> Products = ConfigurationManager.GetProductsAsync(Guid.Parse(Account.AccountId)).Result.Result;

					// build xml list of products
					List<XElement> XmlProducts = new List<XElement>();
					foreach (Product Product in Products)
					{
						string[] PackageFamilyName = Product.PfnId.Split(new char[]{'_' });
						string PackageName = (PackageFamilyName.Length > 1) ? PackageFamilyName[0] : Product.PfnId;
						string PublisherId = (PackageFamilyName.Length > 1) ? PackageFamilyName[1] : "";

						AlternateId StoreId = Product.AlternateIds.FirstOrDefault( id => id.AlternateIdType == AlternateIdType.AppId );
				
						XElement XmlProduct = new XElement("Product");
						XmlProduct.Add( new XElement("PackageDisplayName",     Product.ProductName) );
						XmlProduct.Add( new XElement("PackageFamilyName",      Product.PfnId) );
						XmlProduct.Add( new XElement("PackageName",            PackageName) );
						XmlProduct.Add( new XElement("PublisherId",            PublisherId) );
						XmlProduct.Add( new XElement("TitleId",                Product.TitleId.ToString("X").ToUpperInvariant()) );
						XmlProduct.Add( new XElement("StoreId",                (StoreId == null) ? "" : StoreId.Value) );
						XmlProduct.Add( new XElement("ProductId",              Product.ProductId) );
						XmlProduct.Add( new XElement("PrimaryServiceConfigId", Product.PrimaryServiceConfigId) );
						XmlProduct.Add( new XElement("MSAAppId",               Product.MsaAppId) );
						XmlProduct.Add( new XElement("Tier",                   Product.XboxLiveTier) );
						XmlProduct.Add( new XElement("IsTest",                 Product.IsTest) );					
						XmlProducts.Add(XmlProduct);
					}

					if (XmlProducts.Count == 0)
					{
						Console.Error.WriteLine("no products found");
						return;
					}

					// build xml result
					XDocument XmlResult = new XDocument(
						new XElement("Products", XmlProducts)
						);

					// write the result
					Console.WriteLine(XmlResult.ToString());
					bFinished = true;
				}
				catch (Exception e)
				{
					Console.Error.WriteLine("Cannot query products: {0}", GetExceptionString(e));
					bFinished = true;
				}
			}
		}

		static string GetExceptionString( Exception e )
		{
			if (e is AggregateException)
			{
				string[] InnerMessages = (e as AggregateException).InnerExceptions.Select( innerException => innerException.Message ).ToArray();
				return string.Join( System.Environment.NewLine, InnerMessages );
			}
			else
			{
				return e.Message;
			}
		}

		static Program()
		{
			string GameDK = Environment.GetEnvironmentVariable("GameDK");
			if (string.IsNullOrEmpty(GameDK))
			{
				Console.Error.WriteLine("GameDK environment variable not found - is the GDK installed?");
				return;
			}

			string GDKBinariesFolder = Path.Combine(GameDK, "bin");
			if (!Directory.Exists(GDKBinariesFolder))
			{
				Console.Error.WriteLine($"GDK binaries folder \'{GDKBinariesFolder}\' could not be found - is the GDK installed?");
				return;
			}

			// start out by looking for the assemblies in the GDK binaries folder
			AppDomain.CurrentDomain.AssemblyResolve += (_, args) =>
			{
				string AssemblyName = new AssemblyName(args.Name).Name;
				string AssemblyPath = Path.Combine(GDKBinariesFolder, AssemblyName + ".dll");
				if (File.Exists(AssemblyPath))
				{
					try
					{
						return Assembly.LoadFrom(AssemblyPath);
					}
					catch(Exception)
					{
					}
				}

				return null;
			};
		}

		[STAThread]
		static void Main(string[] args)
		{
			try
			{

				if (args.Length == 0 )
				{
					Console.Error.WriteLine("missing command. Expected \'products\'");
				}
				else if (args[0].Equals("products", StringComparison.InvariantCultureIgnoreCase))
				{
					DumpAllProducts();
				}
				else
				{
					Console.Error.WriteLine("unknown command: {0}", args[0]);
				}
			}
			catch( Exception e )
			{
				Console.Error.WriteLine(e.Message );
			}
		}
	}
}
